{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Procedural Content Generation Framework (PCG) FastGeo Interop",
	"Description": "Extra plugin for Procedural Content Generation Framework which enables runtime spawning of primitives using FastGeo components.",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://docs.unrealengine.com/latest/en-US/procedural-content-generation--framework-in-unreal-engine/",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "PCGFastGeoInterop",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "PCG",
			"Enabled": true
		},
		{
			"Name": "FastGeoStreaming",
			"Enabled": true
		}
	]
}
