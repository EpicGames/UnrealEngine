// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGManagedFastGeoContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PCGManagedFastGeoContainer)

bool UPCGManagedFastGeoContainer::Release(bool bHardRelease, TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPCGManagedFastGeoContainer::Release);

	if (FastGeo)
	{
		UFastGeoContainer::DestroyRuntime(FastGeo);
		FastGeo = nullptr;
	}

	// Can always remove from PCG Component.
	return true;
}
