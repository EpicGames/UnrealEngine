// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PCGManagedResource.h"

#include "FastGeoContainer.h"

#include "PCGManagedFastGeoContainer.generated.h"

struct FPCGProceduralISMComponentDescriptor;

UCLASS()
class UPCGManagedFastGeoContainer : public UPCGManagedResource
{
	GENERATED_BODY()

public:
	//~Begin UPCGManagedResource interface
	virtual bool Release(bool bHardRelease, TSet<TSoftObjectPtr<AActor>>& OutActorsToDelete) override;
	virtual bool ReleaseOnTeardown() const override { return true; }
	//~End UPCGManagedResource interface

	void SetFastGeoContainer(UFastGeoContainer* InFastGeo) { FastGeo = InFastGeo; }

	/** Set references to objects to be kept alive. */
	void SetObjectReferences(TArray<TObjectPtr<UObject>>&& InObjectReferences) { ObjectReferences = MoveTemp(InObjectReferences); }

protected:
	UPROPERTY(Transient)
	TObjectPtr<UFastGeoContainer> FastGeo;

	/** Referenced objects to be kept alive. */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UObject>> ObjectReferences;
};
