// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compute/PrimitiveFactories/PCGPrimitiveFactoryFastGeoPISMC.h"

#include "Components/PCGManagedFastGeoContainer.h"

#include "PCGComponent.h"
#include "PCGContext.h"
#include "PCGModule.h"

#include "FastGeoStreamingModule.h"
#include "PrimitiveSceneInfo.h"
#include "Components/PCGProceduralISMComponentDescriptor.h"
#include "Compute/PCGComputeCommon.h"
#include "Engine/World.h"

bool FPCGPrimitiveFactoryFastGeoPISMC::IsRenderStateCreated() const
{
	if (Components.IsEmpty())
	{
		return true;
	}

	// Registration is driven by the subsystem; wait for the OnRegistered callback.
	if (!*bFastGeoRegistered)
	{
		return false;
	}

	for (int32 PrimitiveIndex = 0; PrimitiveIndex < GetNumPrimitives(); ++PrimitiveIndex)
	{
		FPrimitiveSceneProxy* SceneProxy = GetSceneProxy(PrimitiveIndex);
		if (!SceneProxy || !SceneProxy->GetPrimitiveSceneInfo() || SceneProxy->GetPrimitiveSceneInfo()->GetInstanceSceneDataOffset() == -1)
		{
			return false;
		}
	}

	return true;
}

bool FPCGPrimitiveFactoryFastGeoPISMC::IsAnyRenderStateDirty() const
{
	if (!*bFastGeoRegistered)
	{
		return false;
	}

	for (const FWeakFastGeoComponent& WeakComp : Components)
	{
		if (FFastGeoPrimitiveComponent* Comp = static_cast<FFastGeoPrimitiveComponent*>(WeakComp.Get()))
		{
			if (Comp->IsRenderStateDirty())
			{
				return true;
			}
		}
	}

	return false;
}

FPrimitiveSceneProxy* FPCGPrimitiveFactoryFastGeoPISMC::GetSceneProxy(int32 InPrimitiveIndex) const
{
	if (ensure(Components.IsValidIndex(InPrimitiveIndex)) && Components[InPrimitiveIndex].Get())
	{
		return static_cast<FFastGeoPrimitiveComponent*>(Components[InPrimitiveIndex].Get())->GetSceneProxy();
	}
	else
	{
		return nullptr;
	}
}

int32 FPCGPrimitiveFactoryFastGeoPISMC::GetNumInstances(int32 InPrimitiveIndex, int32 InCellID) const
{
	if (ensure(PrimitiveInfos.IsValidIndex(InPrimitiveIndex)))
	{
		for (const FPCGInstanceRange& PrimitiveInfo : PrimitiveInfos[InPrimitiveIndex].InstanceRanges)
		{
			if (PrimitiveInfo.GetCellID() == InCellID)
			{
				return PrimitiveInfo.GetNumInstances();
			}
		}
	}

	return 0;
}

int32 FPCGPrimitiveFactoryFastGeoPISMC::GetNumInstancesTotal(int32 InPrimitiveIndex) const
{
	int32 NumInstances = 0;

	if (ensure(PrimitiveInfos.IsValidIndex(InPrimitiveIndex)))
	{
		for (const FPCGInstanceRange& PrimitiveInfo : PrimitiveInfos[InPrimitiveIndex].InstanceRanges)
		{
			NumInstances += PrimitiveInfo.GetNumInstances();
		}
	}

	return NumInstances;
}

void FPCGPrimitiveFactoryFastGeoPISMC::Initialize(FParameters&& InParameters)
{
	PrimitiveInfos = MoveTemp(InParameters.PrimitiveInfos);
	CustomPrimitiveData = MoveTemp(InParameters.CustomPrimitiveData);
}

bool FPCGPrimitiveFactoryFastGeoPISMC::Create(FPCGContext* InContext)
{
	IPCGGraphExecutionSource* ExecutionSource = InContext->ExecutionSource.Get();
	if (!ExecutionSource)
	{
		return true;
	}

	UWorld* World = ExecutionSource->GetExecutionState().GetWorld();

	AActor* HitProxyTargetActor = nullptr;
#if WITH_EDITOR
	HitProxyTargetActor = ExecutionSource->GetExecutionState().GetTypedTarget<AActor>();
	// For actor-component-less generation the local execution source has no target actor.
	// The original source (e.g. the UPCGComponent on the user's actor) typically does.
	if (!HitProxyTargetActor)
	{
		if (IPCGGraphExecutionSource* OriginalSource = ExecutionSource->GetExecutionState().GetOriginalSource())
		{
			HitProxyTargetActor = OriginalSource->GetExecutionState().GetTypedTarget<AActor>();
		}
	}
#endif // WITH_EDITOR

	const FString DebugName = ExecutionSource->GetExecutionState().GetDebugName();
	const int32 NumToCreate = FMath::Min(PrimitiveInfos.Num(), PCGComputeConstants::MAX_PRIMITIVE_COMPONENTS_PER_SPAWNER);
	const FBox ExecutionBounds = ExecutionSource->GetExecutionState().GetBounds();

	MeshBounds.SetNumUninitialized(NumToCreate);

	FFastGeoCreateRuntimeResult Result = UFastGeoContainer::CreateRuntime(World, *FString::Printf(TEXT("PCG_%s"), *DebugName), [this, NumToCreate, HitProxyTargetActor, &ExecutionBounds](FFastGeoComponentCluster& InCluster)
	{
		for (int32 DescIndex = 0; DescIndex < NumToCreate; ++DescIndex)
		{
			FPCGPrimitiveInfo& PrimitiveInfo = PrimitiveInfos[DescIndex];
			FPCGProceduralISMComponentDescriptor& Desc = PrimitiveInfo.Descriptor;

			TArray<FInstanceSceneDataBuffers::FCompressedSpatialHashItem> SpatialHashes;
			FBox BoundsOfAllCells(EForceInit::ForceInit);

			PopulateHashes(PrimitiveInfo, ExecutionBounds, SpatialHashes, BoundsOfAllCells);

			if (BoundsOfAllCells.IsValid)
			{
				Desc.WorldBounds = Desc.WorldBounds.Overlap(BoundsOfAllCells);
			}

			FFastGeoProceduralISMComponent& FastGeoComponent = static_cast<FFastGeoProceduralISMComponent&>(InCluster.AddComponent(FFastGeoProceduralISMComponent::Type));
			FastGeoComponent.InitializeFromComponentDescriptor(Desc);
			FastGeoComponent.SetCustomPrimitiveData(CustomPrimitiveData.GetViewForIndex(DescIndex));
			FastGeoComponent.SetSpatialHashes(MoveTemp(SpatialHashes));
#if WITH_EDITOR
			FastGeoComponent.SetHitProxyTargetActor(HitProxyTargetActor);
#endif
			MeshBounds[DescIndex] = Desc.StaticMesh->GetBoundingBox();
		}

		ensure(InCluster.HasComponents());
	}, /*bCollectReferences=*/false); // Disable the default reference collection path as it is currently too costly for runtime use. Instead manually collect references specific to the PISMC below.

	if (!Result.Container)
	{
		UE_LOGF(LogPCG, Error, "Failed to create a runtime FastGeoContainer, geometry will not be created.");
		return true;
	}

	*bFastGeoRegistered = Result.Container->IsFullyRegistered();
	if (!*bFastGeoRegistered)
	{
		Result.Container->GetOnRegistered().AddLambda([Flag = bFastGeoRegistered]() { *Flag = true; });
	}

	for (int32 Index = 0; Index < Result.Components.Num(); ++Index)
	{
		check(PrimitiveInfos.IsValidIndex(Index));
		Components.Add(Result.Components[Index]);
		InstanceCounts.Add(PrimitiveInfos[Index].Descriptor.NumInstances);
	}

	UPCGManagedFastGeoContainer* ManagedPrimitives = FPCGContext::NewObject_AnyThread<UPCGManagedFastGeoContainer>(InContext, Cast<UObject>(ExecutionSource));
	ManagedPrimitives->SetFastGeoContainer(Result.Container);
	ManagedPrimitives->SetObjectReferences(CollectObjectReferences());

	FPCGManagedResourceContainerHelper ContainerHelper(ExecutionSource);
	if (ensure(ContainerHelper.IsValid()))
	{
		ContainerHelper.AddManagedResource(ManagedPrimitives);
	}

	return true;
}

FBox FPCGPrimitiveFactoryFastGeoPISMC::GetMeshBounds(int32 InPrimitiveIndex) const
{
	return ensure(MeshBounds.IsValidIndex(InPrimitiveIndex)) ? MeshBounds[InPrimitiveIndex] : FBox();
}

TArray<TObjectPtr<UObject>> FPCGPrimitiveFactoryFastGeoPISMC::CollectObjectReferences()
{
	TArray<TObjectPtr<UObject>> ObjectReferences;
	ObjectReferences.Reserve(PrimitiveInfos.Num() * 4); // Heuristic

	for (const FPCGPrimitiveInfo& PrimitiveInfo : PrimitiveInfos)
	{
		const FPCGProceduralISMComponentDescriptor& Descriptor = PrimitiveInfo.Descriptor;

		ObjectReferences.Add(Descriptor.StaticMesh.Get());

		if (Descriptor.OverlayMaterial)
		{
			ObjectReferences.Add(Descriptor.OverlayMaterial);
		}

		ObjectReferences.Append(Descriptor.OverrideMaterials);
	}

	return ObjectReferences;
}
