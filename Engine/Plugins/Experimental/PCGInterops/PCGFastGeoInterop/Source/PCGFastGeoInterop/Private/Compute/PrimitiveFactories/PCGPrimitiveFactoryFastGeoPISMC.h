// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PCGProceduralISMComponentDescriptor.h"
#include "Compute/PrimitiveFactories/PCGPrimitiveFactoryISMBase.h"

#include "FastGeoWeakElement.h"
#include "UObject/WeakObjectPtr.h"

struct FPCGContext;
class FPrimitiveSceneProxy;
class UPrimitiveComponent;

class FPCGPrimitiveFactoryFastGeoPISMC : public IPCGPrimitiveFactoryISMBase
{
public:
	//~ Begin IPCGRuntimePrimitiveFactory interface
	virtual bool IsRenderStateCreated() const override;
	virtual int32 GetNumPrimitives() const override { return Components.Num(); }
	virtual FPrimitiveSceneProxy* GetSceneProxy(int32 InPrimitiveIndex) const override;
	virtual bool IsPrimitiveValid(int32 InPrimitiveIndex) const override { return Components.IsValidIndex(InPrimitiveIndex) && Components[InPrimitiveIndex].Get() != nullptr; }
	virtual int32 GetNumInstances(int32 InPrimitiveIndex, int32 InCellID) const override;
	virtual int32 GetNumInstancesTotal(int32 InPrimitiveIndex) const override;
	virtual bool IsAnyRenderStateDirty() const override;
	//~ End IPCGRuntimePrimitiveFactory interface

	//~ Begin IPCGPrimitiveFactoryISMBase interface
	virtual void Initialize(FParameters&& InParameters) override;
	virtual bool Create(FPCGContext* InContext) override;
	virtual FBox GetMeshBounds(int32 InPrimitiveIndex) const override;
	//~ End IPCGPrimitiveFactoryISMBase interface

protected:
	TArray<TObjectPtr<UObject>> CollectObjectReferences();

protected:
	TArray<FPCGPrimitiveInfo> PrimitiveInfos;
	FPCGPackedCustomData CustomPrimitiveData;

	TArray<FWeakFastGeoComponent> Components;
	TArray<int32> InstanceCounts;
	TArray<FBox> MeshBounds;

	// Shared so the OnRegistered lambda can safely write to it even if this factory is destroyed
	// before the container finishes registration.
	TSharedRef<bool> bFastGeoRegistered = MakeShared<bool>(false);
};
