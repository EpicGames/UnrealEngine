// Copyright Epic Games, Inc. All Rights Reserved.

#include "PCGFastGeoInteropModule.h"

#include "Compute/PrimitiveFactories/PCGPrimitiveFactoryFastGeoPISMC.h"

#include "FastGeoWorldSubsystem.h"
#include "PCGCommon.h"
#include "PCGSubsystem.h"

#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FPCGFastGeoInteropModule, PCGFastGeoInterop);

void FPCGFastGeoInteropModule::StartupModule()
{
	PCGPrimitiveFactoryHelpers::Private::SetupFastGeoPrimitiveFactory([]()
	{
		return MakeShared<FPCGPrimitiveFactoryFastGeoPISMC>();
	});

#if WITH_EDITOR
	UFastGeoWorldSubsystem::ComponentsPreRecreateEvent.AddRaw(this, &FPCGFastGeoInteropModule::OnFastGeoComponentsPreRecreate);
#endif
}

void FPCGFastGeoInteropModule::ShutdownModule()
{
#if WITH_EDITOR
	UFastGeoWorldSubsystem::ComponentsPreRecreateEvent.RemoveAll(this);
#endif

	PCGPrimitiveFactoryHelpers::Private::ResetFastGeoPrimitiveFactory();
}

#if WITH_EDITOR
void FPCGFastGeoInteropModule::OnFastGeoComponentsPreRecreate(const TArray<FFastGeoRegisteredComponent>& InComponents) const
{
	if (UPCGSubsystem* PCGSubsystem = UPCGSubsystem::GetSubsystemForCurrentWorld())
	{
		// Could try to be more clever and only refresh the component that was dirtied, but if the instances were flushed we are probably safe to refresh everything.
		PCGSubsystem->DirtyRuntimeGenExecutionSources(EPCGChangeType::None, ERuntimeGenRefreshReason::RenderStateRefresh);
	}
}
#endif
