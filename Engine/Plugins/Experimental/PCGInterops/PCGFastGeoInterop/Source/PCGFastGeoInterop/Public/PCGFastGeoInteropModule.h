// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Stats/Stats.h"

/** Module for experimental plugin that provides access to FastGeo components. To be removed when FastGeo exits experimental. */
class FPCGFastGeoInteropModule final : public IModuleInterface
{
public:
	//~ IModuleInterface implementation
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override { return true; }
	//~ End IModuleInterface implementation

#if WITH_EDITOR
	void OnFastGeoComponentsPreRecreate(const TArray<struct FFastGeoRegisteredComponent>& InComponents) const;
#endif
};
